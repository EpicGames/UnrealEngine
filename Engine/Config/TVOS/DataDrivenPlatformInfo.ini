[DataDrivenPlatformInfo]
bIsConfidential=false
IniParent=IOS
TargetSettingsIniSectionName=/Script/IOSRuntimeSettings.IOSRuntimeSettings
GlobalIdentifier=A478294FBD0D4EC08830B6D4D219C1A4

NormalIconPath=Launcher/TVOS/Platform_TVOS_24x
LargeIconPath=Launcher/TVOS/Platform_TVOS_128x
XLargeIconPath=
AutoSDKPath=
TutorialPath=SharingAndReleasing/Mobile/iOS
Windows:bIsEnabled=true
Mac:bIsEnabled=true
Linux:bIsEnabled=false
bUsesHostCompiler=false
bUATClosesAfterLaunch=true
PlatformGroupName=Mobile
ICUDataVersion=icudt64l

[ShaderPlatform METAL_SM5_TVOS]
Language=Metal
MaxFeatureLevel=SM5
ShaderFormat=SF_METAL_SM5_TVOS
bTargetsTiledGPU=true
bSupportsIndexBufferUAVs=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsDualSourceBlending=true
bSupportsIndependentSamplers=true
bSupportsDistanceFields=true
bSupportsLumenGI=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_TVOS_SM5", "TVOS Metal SM5")

[ShaderPlatform METAL_ES3_1_TVOS]
Language=Metal
MaxFeatureLevel=ES3_1
ShaderFormat=SF_METAL_ES3_1_TVOS
bIsMobile=true
bTargetsTiledGPU=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsShaderPipelines = false
bSupportsIndependentSamplers=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_TVOS_Mobile", "TVOS Metal Mobile")
