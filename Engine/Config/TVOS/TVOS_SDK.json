{
	"//": "inherit the IOS MinSoftwareVersion",
	"ParentSDKFile": "../IOS/IOS_SDK.json"
}
